// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class IasTool : ModuleRules
{
	public IasTool(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("Launch");

		PrivateDependencyModuleNames.Add("Core");
		PrivateDependencyModuleNames.Add("IoStoreOnDemand");
		PrivateDependencyModuleNames.Add("IoStoreOnDemandUtilities");
		PrivateDependencyModuleNames.Add("Projects");
	}
}

