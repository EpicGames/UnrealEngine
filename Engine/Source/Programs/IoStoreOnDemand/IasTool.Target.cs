// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms("Win64", "Linux", "Mac")]
public class IasToolTarget : TargetRules
{
	public IasToolTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;
		LaunchModuleName = "IasTool";
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;

		bBuildDeveloperTools = false;
		bBuildWithEditorOnlyData = false;
		bCompileAgainstApplicationCore = false;
		bCompileAgainstCoreUObject = false;
		bCompileAgainstEngine = false;
		bCompileICU = false;
		bCompileNavmeshClusterLinks = false;
		bCompileNavmeshSegmentLinks = false;
		bCompileRecast = false;
		bCompileWithPluginSupport = true;
		bForceEnableExceptions = true;
		bIsBuildingConsoleApplication = true;
		bUsesSlate = false;
		bWithLiveCoding = false;
		bWithServerCode = false;

		bEnableTrace = true;
	}
}
