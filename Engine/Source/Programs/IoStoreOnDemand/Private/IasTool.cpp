// Copyright Epic Games, Inc. All Rights Reserved.

#include <CoreGlobals.h>
#include <HAL/LowLevelMemTracker.h>
#include <HAL/Platform.h>
#include <LaunchEngineLoop.h>
#include <Modules/ModuleManager.h>
#include <Misc/ScopeExit.h>

#include <RequiredProgramMainCPPInclude.h>

IMPLEMENT_APPLICATION(IasTool, "IasTool");

////////////////////////////////////////////////////////////////////////////////
namespace UE::IoStore::Tool {

int32 Main(int32, TCHAR*[]);

} // namespace UE::IoStore

////////////////////////////////////////////////////////////////////////////////
INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	FTaskTagScope Scope(ETaskTag::EGameThread);

	if (int Ret = GEngineLoop.PreInit(ArgC, ArgV, TEXT("-stdout")); Ret != 0)
	{
		return Ret;
	}

	ON_SCOPE_EXIT {
		LLM(FLowLevelMemTracker::Get().UpdateStatsPerFrame());
		RequestEngineExit(TEXT("Exiting"));
		FEngineLoop::AppPreExit();
		FModuleManager::Get().UnloadModulesAtShutdown();
		FEngineLoop::AppExit();
	};

	return UE::IoStore::Tool::Main(ArgC, ArgV);
}
